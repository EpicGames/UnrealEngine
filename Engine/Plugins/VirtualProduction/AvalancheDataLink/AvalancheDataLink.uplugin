{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Motion Design Data Link Integration",
	"Description": "",
	"Category": "Other",
	"CreatedBy": "",
	"CreatedByURL": "",
	"DocsURL": "",
	"MarketplaceURL": "",
	"CanContainContent": true,
	"IsBetaVersion": true,
	"IsExperimentalVersion": false,
	"Installed": false,
	"Modules": [
		{
			"Name": "AvalancheDataLink",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "AvalancheDataLinkEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "Avalanche",
			"Enabled": true
		},
		{
			"Name": "DataLink",
			"Enabled": true
		}
	]
}