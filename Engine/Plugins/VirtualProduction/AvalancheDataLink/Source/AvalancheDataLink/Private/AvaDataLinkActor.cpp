// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaDataLinkActor.h"
#include "AvaDataLinkInstance.h"
#include "Engine/World.h"

AAvaDataLinkActor::AAvaDataLinkActor()
{
	FWorldDelegates::OnWorldCleanup.AddUObject(this, &AAvaDataLinkActor::OnWorldCleanup);
}

void AAvaDataLinkActor::ExecuteDataLinkInstances()
{
	for (UAvaDataLinkInstance* DataLinkInstance: DataLinkInstances)
	{
		if (DataLinkInstance)
		{
			DataLinkInstance->Execute();
		}
	}
}

void AAvaDataLinkActor::StopDataLinkInstances()
{
	for (UAvaDataLinkInstance* DataLinkInstance: DataLinkInstances)
	{
		if (DataLinkInstance)
		{
			DataLinkInstance->Stop();
		}
	}
}

void AAvaDataLinkActor::BeginPlay()
{
	Super::BeginPlay();

	if (bExecuteOnBeginPlay)
	{
		ExecuteDataLinkInstances();
	}
}

#if WITH_EDITOR
FString AAvaDataLinkActor::GetDefaultActorLabel() const
{
	return TEXT("Motion Design Data Link Actor");
}
#endif

void AAvaDataLinkActor::BeginDestroy()
{
	Super::BeginDestroy();
	StopDataLinkInstances();
}

void AAvaDataLinkActor::OnWorldCleanup(UWorld* InWorld, bool bInSessionEnded, bool bInCleanupResources)
{
	if (InWorld == GetWorld())
	{
		StopDataLinkInstances();
	}
}
