// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaDataLinkInstance.h"
#include "DataLinkExecutor.h"
#include "DataLinkExecutorArguments.h"
#include "Engine/World.h"
#include "RemoteControl/AvaDataLinkRCProcessor.h"

void UAvaDataLinkInstance::Execute()
{
	Stop();

	Executor = FDataLinkExecutor::Create(FDataLinkExecutorArguments(DataLinkInstance)
#if WITH_DATALINK_CONTEXT
		.SetContextName(BuildContextName())
#endif
		.SetContextObject(this)
		.SetOutputProcessors(OutputProcessors));

	Executor->Run();
}

void UAvaDataLinkInstance::Stop()
{
	if (Executor.IsValid())
	{
		Executor->Stop();
		Executor.Reset();
	}
}

#if WITH_EDITORONLY_DATA
void UAvaDataLinkInstance::DeclareConstructClasses(TArray<FTopLevelAssetPath>& OutConstructClasses, const UClass* InSpecificSubclass)
{
	Super::DeclareConstructClasses(OutConstructClasses, InSpecificSubclass);
	OutConstructClasses.Add(FTopLevelAssetPath(UAvaDataLinkRCProcessor::StaticClass()));
}
#endif

void UAvaDataLinkInstance::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITOR
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (!ControllerMappings_DEPRECATED.IsEmpty() && OutputProcessors.IsEmpty())
	{
		UAvaDataLinkRCProcessor* const RCProcessor = NewObject<UAvaDataLinkRCProcessor>(this, NAME_None, GetMaskedFlags(RF_PropagateToSubObjects));
		RCProcessor->ControllerMappings = ControllerMappings_DEPRECATED;
		OutputProcessors.Add(RCProcessor);
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif
}

#if WITH_DATALINK_CONTEXT
FString UAvaDataLinkInstance::BuildContextName() const
{
	return FString::Printf(TEXT("Motion Design Data Link. World: '%s'"), *GetNameSafe(GetTypedOuter<UWorld>()));
}
#endif
