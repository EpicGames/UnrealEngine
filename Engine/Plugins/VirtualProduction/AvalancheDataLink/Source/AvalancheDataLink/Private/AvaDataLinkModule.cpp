// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaDataLinkLog.h"
#include "Modules/ModuleManager.h"

DEFINE_LOG_CATEGORY(LogAvaDataLink);

IMPLEMENT_MODULE(FDefaultModuleImpl, AvalancheDataLink)
