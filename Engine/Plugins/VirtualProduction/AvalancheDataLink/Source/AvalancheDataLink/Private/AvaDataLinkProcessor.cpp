// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaDataLinkProcessor.h"
#include "AvaSceneSubsystem.h"
#include "Engine/Level.h"
#include "Engine/World.h"

IAvaSceneInterface* UAvaDataLinkProcessor::GetSceneInterface() const
{
	ULevel* Level = GetTypedOuter<ULevel>();
	if (!Level || !Level->OwningWorld)
	{
		return nullptr;
	}

	const UAvaSceneSubsystem* SceneSubsystem = Level->OwningWorld->GetSubsystem<UAvaSceneSubsystem>();
	if (!SceneSubsystem)
	{
		// No subsystem yet available, try finding the scene interface by iterating the actors
		return UAvaSceneSubsystem::FindSceneInterface(Level);
	}

	return SceneSubsystem->GetSceneInterface(Level);
}
