// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataLinkProcessor.h"
#include "AvaDataLinkProcessor.generated.h"

class IAvaSceneInterface;

UCLASS(Abstract)
class UAvaDataLinkProcessor : public UDataLinkProcessor
{
	GENERATED_BODY()

protected:
	IAvaSceneInterface* GetSceneInterface() const;
};
