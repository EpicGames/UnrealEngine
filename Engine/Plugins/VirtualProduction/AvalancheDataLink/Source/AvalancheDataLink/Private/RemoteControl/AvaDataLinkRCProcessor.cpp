// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaDataLinkRCProcessor.h"
#include "AvaDataLinkLog.h"
#include "Controller/RCController.h"
#include "DataLinkExecutor.h"
#include "DataLinkJsonUtils.h"
#include "DataLinkUtils.h"
#include "Dom/JsonValue.h"
#include "IAvaSceneInterface.h"
#include "JsonObjectConverter.h"
#include "JsonObjectWrapper.h"
#include "PropertyBindingTypes.h"
#include "RCVirtualPropertyContainer.h"
#include "RemoteControlPreset.h"

namespace UE::AvaDataLink::Private
{
	struct FCopyInfo
	{
		const FProperty* SourceProperty;
		const uint8* SourceMemory;
		FProperty* TargetProperty;
		uint8* TargetMemory;
	};

	/**
	 * Tries promoting a given source property value to match the target property type and copies it to the target memory
	 * Derived from FPropertyBindingBindingCollection::PerformCopy
	 */
	template<typename InSourcePropertyType>
	class FPromotionCopy
	{
		template<typename InTargetPropertyType>
		bool CopySingle(const FCopyInfo& InCopyInfo, const InSourcePropertyType& InSourceProperty)
		{
			using FSourceCppType = InSourcePropertyType::TCppType;
			using FTargetCppType = InTargetPropertyType::TCppType;

			static_assert(std::is_convertible_v<FSourceCppType, FTargetCppType>);

			if (InTargetPropertyType* TargetProperty = CastField<InTargetPropertyType>(InCopyInfo.TargetProperty))
			{
				TargetProperty->SetPropertyValue(InCopyInfo.TargetMemory, InSourceProperty.GetPropertyValue(InCopyInfo.SourceMemory));
				return true;
			}
			return false;
		}

	public:
		/** Tries promoting from a source property type to any of the given target property types */
		template<typename ...InTargetPropertyTypes>
		bool Copy(const FCopyInfo& InCopyInfo)
		{
			if (const InSourcePropertyType* SourceProperty = CastField<InSourcePropertyType>(InCopyInfo.SourceProperty))
			{
				return (this->CopySingle<InTargetPropertyTypes>(InCopyInfo, *SourceProperty) || ...);
			}
			return false;
		}
	};

	bool PromoteCopy(const FCopyInfo& InCopyInfo)
	{
		// Bool Promotions
		return FPromotionCopy<FBoolProperty>().Copy<FByteProperty, FIntProperty, FUInt32Property, FInt64Property, FFloatProperty, FDoubleProperty>(InCopyInfo)
			// Byte Promotion
			|| FPromotionCopy<FByteProperty>().Copy<FIntProperty, FUInt32Property, FInt64Property, FFloatProperty, FDoubleProperty>(InCopyInfo)
			// Int32 Promotions
			|| FPromotionCopy<FIntProperty>().Copy<FInt64Property, FFloatProperty, FDoubleProperty>(InCopyInfo)
			// Uint32 Promotions
			|| FPromotionCopy<FUInt32Property>().Copy<FInt64Property, FFloatProperty, FDoubleProperty>(InCopyInfo)
			// Float Promotions
			|| FPromotionCopy<FFloatProperty>().Copy<FIntProperty, FInt64Property, FDoubleProperty>(InCopyInfo)
			// Double Promotions
			|| FPromotionCopy<FDoubleProperty>().Copy<FIntProperty, FInt64Property, FFloatProperty>(InCopyInfo)
		;
	}

} // UE::AvaDataLink::Private

void UAvaDataLinkRCProcessor::OnProcessOutput(const FDataLinkExecutor& InExecutor, FConstStructView InOutputDataView)
{
	if (!InOutputDataView.IsValid())
	{
		return;
	}

	URemoteControlPreset* const Preset = GetRemoteControlPreset();
	if (!Preset)
	{
		UE_LOGF(LogAvaDataLink, Error, "[%.*ls] Data Link execution finished, but Remote Control is invalid!"
			, InExecutor.GetContextName().Len()
			, InExecutor.GetContextName().GetData());
		return;
	}

	URCVirtualPropertyContainerBase* const ControllerContainer = Preset->GetControllerContainer();
	if (!ControllerContainer)
	{
		UE_LOGF(LogAvaDataLink, Error, "[%.*ls] Data Link execution finished, but Remote Control '%ls' has invalid Controller Container!"
			, InExecutor.GetContextName().Len()
			, InExecutor.GetContextName().GetData()
			, *Preset->GetName());
		return;
	}

	const FStructView TargetDataView = ControllerContainer->GetPropertyBagMutableValue();
	if (!TargetDataView.IsValid())
	{
		UE_LOGF(LogAvaDataLink, Error, "[%.*ls] Data Link execution finished, but Remote Control '%ls' has invalid Controller Container Data View!"
			, InExecutor.GetContextName().Len()
			, InExecutor.GetContextName().GetData()
			, *Preset->GetName());
		return;
	}

	TArray<URCController*> ModifiedControllers;
	ModifiedControllers.Reserve(ControllerMappings.Num());

	if (const FJsonObjectWrapper* OutputJson = InOutputDataView.GetPtr<const FJsonObjectWrapper>())
	{
		if (!OutputJson->JsonObject.IsValid())
		{
			UE_LOGF(LogAvaDataLink, Warning, "[%.*ls] Data Link output could not be applied to controllers. Json Object was not valid!"
				, InExecutor.GetContextName().Len()
				, InExecutor.GetContextName().GetData());
			return;
		}

		TSharedRef<FJsonObject> JsonObject = OutputJson->JsonObject.ToSharedRef();

		ForEachResolvedController(InExecutor, Preset, TargetDataView,
			[&JsonObject, &ModifiedControllers](const FResolvedController& InResolvedController)
			{
				TSharedPtr<FJsonValue> SourceJsonValue = UE::DataLinkJson::FindJsonValue(JsonObject, InResolvedController.Mapping->OutputFieldName);

				// If property was set, return true to signal that the controller value was modified
				if (FJsonObjectConverter::JsonValueToUProperty(SourceJsonValue, InResolvedController.TargetProperty, InResolvedController.TargetMemory))
				{
					ModifiedControllers.AddUnique(InResolvedController.Controller);
				}
			});
	}
	else
	{
		ForEachResolvedController(InExecutor, Preset, TargetDataView,
			[&InOutputDataView, &ModifiedControllers, &InExecutor](const FResolvedController& InResolvedController)
			{
				FString ErrorMessage;
				const UE::DataLink::FConstPropertyView SourceView = UE::DataLink::ResolveConstPropertyView(InOutputDataView, InResolvedController.Mapping->OutputFieldName, &ErrorMessage);
				if (!SourceView.Property)
				{
					UE_LOGF(LogAvaDataLink, Warning, "[%.*ls] Data Link output field name '%ls' could not be applied: %ls"
						, InExecutor.GetContextName().Len()
						, InExecutor.GetContextName().GetData()
						, *InResolvedController.Mapping->OutputFieldName
						, *ErrorMessage);
					return;
				}

				const UE::PropertyBinding::EPropertyCompatibility Compatibility = UE::PropertyBinding::GetPropertyCompatibility(SourceView.Property, InResolvedController.TargetProperty);
				if (Compatibility == UE::PropertyBinding::EPropertyCompatibility::Incompatible)
				{
					UE_LOGF(LogAvaDataLink, Warning, "[%.*ls] Data Link output '%ls' could not be applied to controller '%ls' as types are incompatible"
						, InExecutor.GetContextName().Len()
						, InExecutor.GetContextName().GetData()
						, *InResolvedController.Mapping->OutputFieldName
						, *InResolvedController.Mapping->TargetController.Name.ToString());
					return;
				}

				switch (Compatibility)
				{
				case UE::PropertyBinding::EPropertyCompatibility::Compatible:
					InResolvedController.TargetProperty->CopyCompleteValue(InResolvedController.TargetMemory, SourceView.Memory);
					ModifiedControllers.AddUnique(InResolvedController.Controller);
					break;

				case UE::PropertyBinding::EPropertyCompatibility::Promotable:
					ensureMsgf(UE::AvaDataLink::Private::PromoteCopy({
						.SourceProperty = SourceView.Property,
						.SourceMemory = SourceView.Memory,
						.TargetProperty = InResolvedController.TargetProperty,
						.TargetMemory = InResolvedController.TargetMemory
					}), TEXT("Promotion failed even though compatibility was deemed as 'promotable'."));
					ModifiedControllers.AddUnique(InResolvedController.Controller);
					break;
				}
			});
	}

	TSet<FGuid> ModifiedControllerIds;
	ModifiedControllerIds.Reserve(ModifiedControllers.Num());

	for (URCController* Controller : ModifiedControllers)
	{
		Controller->OnModifyPropertyValue();
		ModifiedControllerIds.Add(Controller->Id);
	}

	Preset->OnControllerModified().Broadcast(Preset, ModifiedControllerIds);
}

URemoteControlPreset* UAvaDataLinkRCProcessor::GetRemoteControlPreset() const
{
	if (IAvaSceneInterface* SceneInterface = GetSceneInterface())
	{
		return SceneInterface->GetRemoteControlPreset();
	}
	return nullptr;
}

void UAvaDataLinkRCProcessor::ForEachResolvedController(const FDataLinkExecutor& InExecutor, URemoteControlPreset* InPreset, FStructView InTargetDataView, TFunctionRef<void(const FResolvedController&)> InFunction)
{
	for (const FAvaDataLinkControllerMapping& Mapping : ControllerMappings)
	{
		FResolvedController ResolvedMapping;
		ResolvedMapping.Mapping = &Mapping;

		// Get the controller to retrieve the underlying Property FName of the controller within the Controller Container Property Bag
		ResolvedMapping.Controller = Cast<URCController>(Mapping.TargetController.FindController(InPreset));
		if (!ResolvedMapping.Controller)
		{
			UE_LOGF(LogAvaDataLink, Warning, "[%.*ls] Data Link output '%ls' could not be applied to controller '%ls'. Controller was not found in preset '%ls'!"
				, InExecutor.GetContextName().Len()
				, InExecutor.GetContextName().GetData()
				, *Mapping.OutputFieldName
				, *Mapping.TargetController.Name.ToString()
				, *InPreset->GetName());
			continue;
		}

		const UE::DataLink::FPropertyView TargetPropertyView = UE::DataLink::ResolvePropertyView(InTargetDataView, ResolvedMapping.Controller->PropertyName.ToString());
		if (!TargetPropertyView.Property)
		{
			UE_LOGF(LogAvaDataLink, Warning, "[%.*ls] Data Link output '%ls' could not be applied to controller '%ls'. Controller property '%ls' in preset '%ls' was not found!"
				, InExecutor.GetContextName().Len()
				, InExecutor.GetContextName().GetData()
				, *Mapping.OutputFieldName
				, *Mapping.TargetController.Name.ToString()
				, *ResolvedMapping.Controller->PropertyName.ToString()
				, *InPreset->GetName());
			continue;
		}

		ResolvedMapping.TargetProperty = TargetPropertyView.Property;
		ResolvedMapping.TargetMemory = TargetPropertyView.Memory;
		InFunction(ResolvedMapping);
	}
}
