// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaDataLinkProcessor.h"
#include "RemoteControl/AvaDataLinkControllerMapping.h"
#include "AvaDataLinkRCProcessor.generated.h"

class FProperty;
class URCController;
class URemoteControlPreset;
struct FAvaDataLinkControllerMapping;

UCLASS(DisplayName="Motion Design Data Link Remote Control Processor")
class UAvaDataLinkRCProcessor : public UAvaDataLinkProcessor
{
	GENERATED_BODY()

protected:
	//~ Begin UDataLinkProcessor
	virtual void OnProcessOutput(const FDataLinkExecutor& InExecutor, FConstStructView InOutputDataView) override;
	//~ End UDataLinkProcessor

	URemoteControlPreset* GetRemoteControlPreset() const;

	struct FResolvedController
	{
		const FAvaDataLinkControllerMapping* Mapping;
		TObjectPtr<URCController> Controller;
		FProperty* TargetProperty;
		uint8* TargetMemory;
	};

	/** Iterates each controller resolved from the controller mappings */
	void ForEachResolvedController(const FDataLinkExecutor& InExecutor, URemoteControlPreset* InPreset, FStructView InTargetDataView, TFunctionRef<void(const FResolvedController&)> InFunction);

private:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Motion Design Data Link", DisplayName="Output Field to Controller Mappings", meta=(AllowPrivateAccess="true"))
	TArray<FAvaDataLinkControllerMapping> ControllerMappings;

	friend class UAvaDataLinkInstance;
};
