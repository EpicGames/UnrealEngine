// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFramework/Actor.h"
#include "AvaDataLinkActor.generated.h"

class UAvaDataLinkInstance;

UCLASS(MinimalAPI, DisplayName="Motion Design Data Link Actor", HideCategories=(Activation, Actor, AssetUserData, Collision, Cooking, DataLayers, HLOD, Input, LevelInstance, Mobility, Navigation, Networking, Physics, Rendering, Replication, Tags, Transform, WorldPartition))
class AAvaDataLinkActor : public AActor
{
	GENERATED_BODY()

public:
	AAvaDataLinkActor();

	UFUNCTION(BlueprintCallable, CallInEditor, Category="Motion Design Data Link")
	void ExecuteDataLinkInstances();

	UFUNCTION(BlueprintCallable, CallInEditor, Category="Motion Design Data Link")
	void StopDataLinkInstances();

	//~ Begin AActor
	virtual void BeginPlay() override;
#if WITH_EDITOR
	virtual FString GetDefaultActorLabel() const override;
#endif
	//~ End AActor

	//~ Begin UObject
	virtual void BeginDestroy() override;
	//~ End UObject

	void OnWorldCleanup(UWorld* InWorld, bool bInSessionEnded, bool bInCleanupResources);

	UPROPERTY(EditAnywhere, Instanced, Category="Motion Design Data Link")
	TArray<TObjectPtr<UAvaDataLinkInstance>> DataLinkInstances;

	/** Whether to automatically execute the data link instances on Begin Play */
	UPROPERTY(EditAnywhere, Category="Motion Design Data Link")
	bool bExecuteOnBeginPlay = false;
};
