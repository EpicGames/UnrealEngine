// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataLinkInstance.h"
#include "RemoteControl/AvaDataLinkControllerMapping.h"
#include "UObject/Object.h"
#include "AvaDataLinkInstance.generated.h"

#define UE_API AVALANCHEDATALINK_API

class FDataLinkExecutor;
class UDataLinkProcessor;

UCLASS(MinimalAPI, EditInlineNew, DisplayName="Motion Design Data Link Instance")
class UAvaDataLinkInstance : public UObject
{
	GENERATED_BODY()

public:
 	/** Starts a new active data link execution, stopping the existing if any */
	UE_API void Execute();

	/** Stops the active data link execution if any */
	UE_API void Stop();

	static FName GetDataLinkInstancePropertyName()
	{
		return GET_MEMBER_NAME_CHECKED(UAvaDataLinkInstance, DataLinkInstance);
	}

	static FName GetOutputProcessorsPropertyName()
	{
		return GET_MEMBER_NAME_CHECKED(UAvaDataLinkInstance, OutputProcessors);
	}

	//~ Begin UObject
#if WITH_EDITORONLY_DATA
	static UE_API void DeclareConstructClasses(TArray<FTopLevelAssetPath>& OutConstructClasses, const UClass* InSpecificSubclass);
#endif
	UE_API virtual void PostLoad() override;
	//~ End UObject

private:
#if WITH_DATALINK_CONTEXT
	FString BuildContextName() const;
#endif

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Motion Design Data Link", meta=(AllowPrivateAccess="true"))
	FDataLinkInstance DataLinkInstance;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Instanced, Category="Motion Design Data Link", meta=(AllowPrivateAccess="true"))
	TArray<TObjectPtr<UDataLinkProcessor>> OutputProcessors;

#if WITH_EDITORONLY_DATA
	UE_DEPRECATED(5.7, "ControllerMappings in Data Link Instance is deprecated. Use Output Processors instead")
	UPROPERTY()
	TArray<FAvaDataLinkControllerMapping> ControllerMappings_DEPRECATED;
#endif

	TSharedPtr<FDataLinkExecutor> Executor;
};

#undef UE_API
