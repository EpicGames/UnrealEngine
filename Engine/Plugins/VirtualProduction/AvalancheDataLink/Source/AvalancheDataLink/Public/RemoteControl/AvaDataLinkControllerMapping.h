// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaRCControllerId.h"
#include "AvaDataLinkControllerMapping.generated.h"

USTRUCT(BlueprintType, DisplayName="Motion Design Data Link RC Controller Mapping")
struct FAvaDataLinkControllerMapping
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Motion Design Data Link")
	FString OutputFieldName;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Motion Design Data Link")
	FAvaRCControllerId TargetController;
};
