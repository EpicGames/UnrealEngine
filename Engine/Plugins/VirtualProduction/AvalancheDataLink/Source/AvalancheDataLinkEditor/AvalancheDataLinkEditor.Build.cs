// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class AvalancheDataLinkEditor : ModuleRules
{
    public AvalancheDataLinkEditor(ReadOnlyTargetRules Target) : base(Target)
    {
        PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

        PrivateDependencyModuleNames.AddRange(
            new string[]
            {
                "AvalancheDataLink",
                "AvalancheInteractiveTools",
                "Core",
                "CoreUObject",
                "Engine",
                "InteractiveToolsFramework",
                "PropertyEditor",
                "Slate",
                "SlateCore",
                "UnrealEd",
            }
        );
    }
}
