// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaDataLinkActorTool.h"
#include "AvaDataLinkActor.h"
#include "AvaDataLinkEditorCommands.h"
#include "Builders/AvaInteractiveToolsToolBuilder.h"
#include "Styling/SlateIconFinder.h"

UAvaDataLinkActorTool::UAvaDataLinkActorTool()
{
	ActorClass = AAvaDataLinkActor::StaticClass();
}

bool UAvaDataLinkActorTool::ShouldForceDefaultAction() const
{
	// Force to always use default action since Data Link Actor has no transform
	return true;
}

bool UAvaDataLinkActorTool::SupportsDefaultAction() const
{
	return true;
}

bool UAvaDataLinkActorTool::OnBegin()
{
	// Overriden since default OnBegin returns false on if ViewportPlannerClass is null
	BeginTransaction();
	return true;
}

void UAvaDataLinkActorTool::OnRegisterTool(IAvalancheInteractiveToolsModule* InAITModule)
{
	Super::OnRegisterTool(InAITModule);

	FAvaInteractiveToolsToolParameters ToolParameters = CreateDefaultToolParameters();
	ToolParameters.UICommand = UE::AvaDataLink::FEditorCommands::Get().DataLinkActorTool;

	InAITModule->RegisterTool(IAvalancheInteractiveToolsModule::CategoryNameActor, MoveTemp(ToolParameters));
}
