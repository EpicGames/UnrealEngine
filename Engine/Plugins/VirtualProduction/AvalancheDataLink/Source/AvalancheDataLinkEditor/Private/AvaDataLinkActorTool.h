// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Tools/AvaInteractiveToolsActorToolBase.h"
#include "AvaDataLinkActorTool.generated.h"

UCLASS()
class UAvaDataLinkActorTool : public UAvaInteractiveToolsActorToolBase
{
	GENERATED_BODY()

public:
	UAvaDataLinkActorTool();

	//~ Begin UAvaInteractiveToolsToolBase
	virtual bool ShouldForceDefaultAction() const override;
	virtual bool SupportsDefaultAction() const override;
	virtual bool OnBegin() override;
	virtual void OnRegisterTool(IAvalancheInteractiveToolsModule* InAITModule) override;
	//~ End UAvaInteractiveToolsToolBase
};
