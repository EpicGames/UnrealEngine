// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaDataLinkEditorCommands.h"
#include "Styling/AppStyle.h"

#define LOCTEXT_NAMESPACE "AvaDataLinkEditorCommands"

namespace UE::AvaDataLink
{

FEditorCommands::FEditorCommands()
	: TCommands(TEXT("AvaDataLinkEditor")
	, LOCTEXT("AvaDataLinkEditor", "Motion Design Data Link Editor")
	, NAME_None
	, FAppStyle::GetAppStyleSetName())
{
}

void FEditorCommands::RegisterCommands()
{
	FUICommandInfo::MakeCommandInfo(SharedThis(this)
		, DataLinkActorTool
		, TEXT("DataLinkActorTool")
		, LOCTEXT("DataLinkActorTool", "Data Link Actor")
		, LOCTEXT("DataLinkActorTool_ToolTip", "Create a Motion Design Data Link Actor")
		, FSlateIcon(FAppStyle::GetAppStyleSetName(), "ClassIcon.Actor")
		, EUserInterfaceActionType::ToggleButton
		, FInputChord()
	);
}

} // UE::AvaDataLink

#undef LOCTEXT_NAMESPACE
