// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"

namespace UE::AvaDataLink
{

class FEditorCommands : public TCommands<FEditorCommands>
{
public:
	FEditorCommands();

	//~ Begin TCommands
	virtual void RegisterCommands() override;
	//~ End TCommands

	/** Command used to spawn the Motion Design Data Link Actor */
	TSharedPtr<FUICommandInfo> DataLinkActorTool;
};

} // UE::AvaDataLink
