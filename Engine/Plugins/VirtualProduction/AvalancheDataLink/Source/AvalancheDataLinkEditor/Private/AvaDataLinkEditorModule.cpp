// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaDataLinkEditorModule.h"
#include "AvaDataLinkEditorCommands.h"
#include "AvaDataLinkInstance.h"
#include "DetailsView/AvaDataLinkInstanceCustomization.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"

IMPLEMENT_MODULE(FAvaDataLinkEditorModule, AvalancheDataLinkEditor)

void FAvaDataLinkEditorModule::StartupModule()
{
	UE::AvaDataLink::FEditorCommands::Register();

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");

	PropertyEditorModule.RegisterCustomClassLayout(CustomizedClasses.Add_GetRef(UAvaDataLinkInstance::StaticClass()->GetFName())
		, FOnGetDetailCustomizationInstance::CreateStatic(&FAvaDataLinkInstanceCustomization::MakeInstance));
}

void FAvaDataLinkEditorModule::ShutdownModule()
{
	UE::AvaDataLink::FEditorCommands::Unregister();

	if (FPropertyEditorModule* PropertyEditorModule = FModuleManager::GetModulePtr<FPropertyEditorModule>("PropertyEditor"))
	{
		for (FName CustomizedClass : CustomizedClasses)
		{
			PropertyEditorModule->UnregisterCustomClassLayout(CustomizedClass);
		}
		CustomizedClasses.Reset();
	}
}
