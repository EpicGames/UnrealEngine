// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Modules/ModuleInterface.h"
#include "UObject/NameTypes.h"

class FAvaDataLinkEditorModule : public IModuleInterface
{
    //~ Begin IModuleInterface
    virtual void StartupModule() override;
    virtual void ShutdownModule() override;
    //~ End IModuleInterface

    TArray<FName> CustomizedClasses;
};
