// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaDataLinkControllerMappingsBuilder.h"
#include "AvaDataLinkInstance.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "IDetailChildrenBuilder.h"

#define LOCTEXT_NAMESPACE "AvaDataLinkControllerMappingsBuilder"

FAvaDataLinkControllerMappingsBuilder::FAvaDataLinkControllerMappingsBuilder(const TSharedRef<IPropertyHandle>& InControllerMappingsHandle, TWeakPtr<IDetailsView> InDetailsViewWeak)
	: ControllerMappingsHandle(InControllerMappingsHandle)
	, ArrayHandle(InControllerMappingsHandle->AsArray().ToSharedRef())
	, DetailsViewWeak(InDetailsViewWeak)
{
}

FName FAvaDataLinkControllerMappingsBuilder::GetName() const
{
	return ControllerMappingsHandle->GetProperty()->GetFName();
}

void FAvaDataLinkControllerMappingsBuilder::SetOnRebuildChildren(FSimpleDelegate InOnRegenerateChildren)
{
	OnRegenerateChildren = InOnRegenerateChildren;
}

void FAvaDataLinkControllerMappingsBuilder::GenerateHeaderRowContent(FDetailWidgetRow& InNodeRow)
{
	FUIAction CopyAction;
	FUIAction PasteAction;
	ControllerMappingsHandle->CreateDefaultPropertyCopyPasteActions(CopyAction, PasteAction);

	InNodeRow
		.PropertyHandleList({ ControllerMappingsHandle })
		.FilterString(ControllerMappingsHandle->GetPropertyDisplayName())
		.NameContent()
		[
			ControllerMappingsHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		[
			ControllerMappingsHandle->CreatePropertyValueWidget()
		]
		.CopyAction(CopyAction)
		.PasteAction(PasteAction);
}

void FAvaDataLinkControllerMappingsBuilder::GenerateChildContent(IDetailChildrenBuilder& InChildrenBuilder)
{
	ArrayHandle->GetNumElements(CachedNumChildren);

	TSharedPtr<IDetailsView> DetailsViewShared = DetailsViewWeak.Pin();
	IDetailsView* DetailsView = DetailsViewShared.Get();

	for (uint32 ChildIndex = 0; ChildIndex < CachedNumChildren; ++ChildIndex)
	{
		TSharedRef<IPropertyHandle> ElementHandle = ArrayHandle->GetElement(ChildIndex);

		TSharedPtr<IPropertyHandle> OutputFieldNameHandle = ElementHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAvaDataLinkControllerMapping, OutputFieldName));
		TSharedPtr<IPropertyHandle> TargetControllerHandle = ElementHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAvaDataLinkControllerMapping, TargetController));

		constexpr bool bDisplayDefaultPropertyButtons = false;

		InChildrenBuilder
			.AddProperty(ElementHandle)
			.CustomWidget()
			.NameContent()
			[
				ElementHandle->CreatePropertyNameWidget()
			]
			.ValueContent()
			.MinDesiredWidth(300)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.FillWidth(1.f)
				[
					OutputFieldNameHandle->CreatePropertyValueWidget(bDisplayDefaultPropertyButtons)
				]
				+ SHorizontalBox::Slot()
				.FillWidth(1.f)
				.Padding(5, 0, 0, 0)
				[
					TargetControllerHandle->CreatePropertyValueWidgetWithCustomization(DetailsView)
				]
			];
	}
}

TSharedPtr<IPropertyHandle> FAvaDataLinkControllerMappingsBuilder::GetPropertyHandle() const
{
	return ControllerMappingsHandle;
}

void FAvaDataLinkControllerMappingsBuilder::Tick(float InDeltaTime)
{
	uint32 NumChildren = 0;
	if (ControllerMappingsHandle->GetNumChildren(NumChildren) != FPropertyAccess::Success)
	{
		return;
	}

	if (NumChildren != CachedNumChildren)
	{
		OnRegenerateChildren.ExecuteIfBound();
	}
}

#undef LOCTEXT_NAMESPACE
