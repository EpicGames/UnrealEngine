// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomNodeBuilder.h"

class IDetailsView;
class IPropertyHandleArray;

class FAvaDataLinkControllerMappingsBuilder : public IDetailCustomNodeBuilder
{
public:
	explicit FAvaDataLinkControllerMappingsBuilder(const TSharedRef<IPropertyHandle>& InControllerMappingsHandle, TWeakPtr<IDetailsView> InDetailsViewWeak);

	//~ Begin IDetailCustomNodeBuilder
	virtual FName GetName() const override;
	virtual void SetOnRebuildChildren(FSimpleDelegate InOnRegenerateChildren) override;
	virtual void GenerateHeaderRowContent(FDetailWidgetRow& InNodeRow) override;
	virtual void GenerateChildContent(IDetailChildrenBuilder& InChildrenBuilder) override;
	virtual TSharedPtr<IPropertyHandle> GetPropertyHandle() const override;
	virtual bool InitiallyCollapsed() const { return false; }
	virtual bool RequiresTick() const override { return true; }
	virtual void Tick(float InDeltaTime) override;
	//~ End IDetailCustomNodeBuilder

private:
	TSharedRef<IPropertyHandle> ControllerMappingsHandle;

	TSharedRef<IPropertyHandleArray> ArrayHandle;

	FSimpleDelegate OnRegenerateChildren;
	
	TWeakPtr<IDetailsView> DetailsViewWeak;

	uint32 CachedNumChildren = 0;
};
