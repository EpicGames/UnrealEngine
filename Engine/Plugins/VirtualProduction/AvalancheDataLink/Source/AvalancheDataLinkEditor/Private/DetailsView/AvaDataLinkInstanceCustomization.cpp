// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaDataLinkInstanceCustomization.h"
#include "AvaDataLinkInstance.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "IDetailChildrenBuilder.h"
#include "PropertyCustomizationHelpers.h"

void FAvaDataLinkInstanceCustomization::CustomizeDetails(IDetailLayoutBuilder& InDetailBuilder)
{
	TSharedRef<IPropertyHandle> DataLinkInstanceHandle = InDetailBuilder.GetProperty(UAvaDataLinkInstance::GetDataLinkInstancePropertyName());
	DataLinkInstanceHandle->MarkHiddenByCustomization();

	TSharedRef<IPropertyHandle> OutputProcessorsHandle = InDetailBuilder.GetProperty(UAvaDataLinkInstance::GetOutputProcessorsPropertyName());
	OutputProcessorsHandle->MarkHiddenByCustomization();

	IDetailCategoryBuilder& Category = InDetailBuilder.EditCategory(DataLinkInstanceHandle->GetDefaultCategoryName(), DataLinkInstanceHandle->GetDefaultCategoryText());
	Category.AddProperty(DataLinkInstanceHandle);
	Category.AddProperty(OutputProcessorsHandle);
}
